{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mass Insights",
	"Description": "Plugin to gather insights into Mass execution",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
	{
		"Name": "MassInsightsAnalysis",
		"Type": "EditorAndProgram",
		"LoadingPhase": "Default",
		"ProgramAllowList": [
		"UnrealInsights"
			]
	},
	{
		"Name": "MassInsightsUI",
		"Type": "EditorAndProgram",
		"LoadingPhase": "Default",
		"ProgramAllowList": [
		"UnrealInsights"
			]
	}
	]
}