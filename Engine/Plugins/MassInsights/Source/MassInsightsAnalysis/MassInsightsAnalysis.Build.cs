// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassInsightsAnalysis : ModuleRules
	{
		public MassInsightsAnalysis(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"MassDeveloper",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
			});
		}
	}
}

