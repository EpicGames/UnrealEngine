// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassInsightsAnalysisModule.h"

#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Trace/MassTraceTypes.h"

namespace MassInsightsAnalysis
{

void FMassInsightsAnalysisModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	UE::Mass::Trace::SetupMassTraceAnalysis(InSession);
}

void FMassInsightsAnalysisModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = TEXT("MassInsightsProvider");
	OutModuleInfo.DisplayName = TEXT("MassInsights");
}

void FMassInsightsAnalysisModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("MassTrace"));
}

void FMassInsightsAnalysisModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, this);
}

void FMassInsightsAnalysisModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, this);
}

} //namespace MassInsightsAnalysis

IMPLEMENT_MODULE(MassInsightsAnalysis::FMassInsightsAnalysisModule, MassInsightsAnalysis);
