// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "TraceServices/ModuleService.h"

namespace MassInsightsAnalysis
{
/**
 *  Class to create and register the Analyzer and Provider for MassInsights.
 *  Implements both a plugin module and the TraceServices::IModule modular feature.
 */
class FMassInsightsAnalysisModule : public IModuleInterface, TraceServices::IModule
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	public:
	//~ Begin TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void GetLoggers(TArray<const TCHAR*>& OutLoggers) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	//~ End TraceServices::IModule interface
};
	
} //namespace MassInsightsAnalysis
