// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassInsightsUI : ModuleRules
	{
		public MassInsightsUI(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SlateCore",
				"Slate",
				"InputCore",
				"TraceInsights",
				"TraceInsightsCore",
				"TraceServices",
				"MassDeveloper",
				"MassInsightsAnalysis",
			});
		}
	}
}

