// Copyright Epic Games, Inc. All Rights Reserved.
#include "MassInsightsTimingTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "Common/ProviderLock.h"

// TraceInsights
#include "MassInsightsUIModule.h"
#include "Framework/Commands/Commands.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "MassInsightsUI/Widgets/SMassInsightsAnalysisTab.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassTraceTypes.h"

#define LOCTEXT_NAMESPACE "RegionsTimingTrack"

class IUnrealInsightsModule;
class FTimingEvent;

namespace MassInsightsUI
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFileActivityTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FMassInsightsViewCommands::FMassInsightsViewCommands()
: TCommands<FMassInsightsViewCommands>(
	TEXT("FMassInsightsViewCommands"),
	NSLOCTEXT("Contexts", "FMassInsightsViewCommands", "Insights - Timing View - Mass Processor"),
	NAME_None,
	TEXT("InsightsStyle"))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMassInsightsViewCommands::~FMassInsightsViewCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
	
UE_DISABLE_OPTIMIZATION_SHIP
void FMassInsightsViewCommands::RegisterCommands()
{
	UI_COMMAND(ShowHideRegionTrack,
		"Mass Processor Phase Tracks",
		"Shows/hides the Tracks demarcating the begin and end of MassProcessor phases.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::Y));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	BindCommands();
	MassInsightsTrack.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	MassInsightsTrack.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::Tick(UE::Insights::Timing::ITimingViewSession& InSession,
	const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (!MassInsightsTrack.IsValid())
	{
		MassInsightsTrack = MakeShared<FMassInsightsTrack>(*this);
		MassInsightsTrack->SetOrder(FTimingTrackOrder::First);
		MassInsightsTrack->SetVisibilityFlag(true);
		InSession.AddScrollableTrack(MassInsightsTrack);
	}

	if (TSharedPtr<MassInsights::SMassInsightsAnalysisTab> Analysis = FMassInsightsUIModule::Get().GetAnalysisTab())
	{
		Analysis->SetSession(&InSession, &InAnalysisSession);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::ShowHideRegionsTrack()
{
	bShowHideRegionsTrack = !bShowHideRegionsTrack;

	if (MassInsightsTrack.IsValid())
	{
		MassInsightsTrack->SetVisibilityFlag(bShowHideRegionsTrack);
	}

	if (bShowHideRegionsTrack)
	{
		MassInsightsTrack->SetDirtyFlag();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::ExtendOtherTracksFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession,
                                                            FMenuBuilder& InOutMenuBuilder)
{
	InOutMenuBuilder.BeginSection("Mass Processor Regions", LOCTEXT("ContextMenu_Section_Regions", "Mass Processor Regions"));
	InOutMenuBuilder.AddMenuEntry(FMassInsightsViewCommands::Get().ShowHideRegionTrack);
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMassInsightsSharedState::BindCommands()
{
	FMassInsightsViewCommands::Register();	
}

//////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMassInsightsTrack)

FMassInsightsTrack::FMassInsightsTrack(FMassInsightsSharedState& InSharedState)
	 : FTimingEventsTrack(TEXT("Mass Phases")), SharedState(InSharedState)
{
	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	AnalysisSession = UnrealInsightsModule.GetAnalysisSession();
}

FMassInsightsTrack::~FMassInsightsTrack()
{
}

void FMassInsightsTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FTimingEventsTrack::BuildContextMenu(MenuBuilder);
}

//////////////////////////////////////////////////////////////////////////

void FMassInsightsTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

		auto MatchEvent = [this, &TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
		{
			return InDepth == TooltipEvent.GetDepth()
				&& InStartTime == TooltipEvent.GetStartTime()
				&& InEndTime == TooltipEvent.GetEndTime();
		};

		FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
		FindRegionEvent(SearchParameters, [this, &InOutTooltip, &TooltipEvent] (
			double InFoundStartTime,
			double InFoundEndTime,
			uint32 InFoundDepth,
			const UE::Mass::Trace::FRegion& InRegion)
		{
			InOutTooltip.Reset();
			InOutTooltip.AddTitle(InRegion.Text, FLinearColor::White);
			InOutTooltip.AddNameValueTextLine(TEXT("Duration:"),  UE::Insights::FormatTimeAuto(InRegion.EndTime-InRegion.BeginTime));
			InOutTooltip.AddNameValueTextLine(TEXT("Depth:"),  FString::FromInt(InRegion.Depth));
			InOutTooltip.UpdateLayout();
		});
	}
}

void FMassInsightsTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder,
	const ITimingTrackUpdateContext& Context)
{
	using namespace UE::Mass::Trace;

	if (AnalysisSession.IsValid())
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();

		const IMassTraceProvider& RegionProvider = ReadMassTraceProvider(*AnalysisSession);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		// we're counting only non-empty lanes, so we can collapse empty ones in the visualization.
		int32 CurDepth = 0;
		RegionProvider.EnumerateLanes([this, Viewport, &CurDepth, &Builder](const FRegionLane& Lane, const int32 Depth)
		{
			bool RegionHadEvents = false;
			Lane.EnumerateRegions(Viewport.GetStartTime(), Viewport.GetEndTime(), [&Builder, &RegionHadEvents, &CurDepth](const FRegion& Region) -> bool
			{
				RegionHadEvents = true;
				Builder.AddEvent(Region.BeginTime, Region.EndTime,CurDepth, Region.Text);
				return true;
			});

			if (RegionHadEvents) CurDepth++;
		});
	}
}

//////////////////////////////////////////////////////////////////////////

void FMassInsightsTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	using namespace UE::Mass::Trace;

	const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
	if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
	{
		bool bFilterOnlyByEventType = false; // this is the most often use case, so the below code tries to optimize it
		TCHAR* FilterEventType = 0;
		if (EventFilterPtr->Is<FTimingEventFilterByEventType>())
		{
			bFilterOnlyByEventType = true;
			const FTimingEventFilterByEventType& EventFilter = EventFilterPtr->As<FTimingEventFilterByEventType>();
			FilterEventType = reinterpret_cast<TCHAR*>(EventFilter.GetEventType());
		}

		if (AnalysisSession.IsValid())
		{
			const IMassTraceProvider& RegionProvider = ReadMassTraceProvider(*AnalysisSession);
			TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

			const FTimingTrackViewport& Viewport = Context.GetViewport();

			if (bFilterOnlyByEventType)
			{
				int32 CurDepth = 0;
				RegionProvider.EnumerateLanes([this, Viewport, &CurDepth, &Builder, FilterEventType](const FRegionLane& Lane, const int32 Depth)
					{
						bool RegionHadEvents = false;
						Lane.EnumerateRegions(Viewport.GetStartTime(), Viewport.GetEndTime(), [&Builder, &RegionHadEvents, &CurDepth, FilterEventType](const FRegion& Region) -> bool
							{
								RegionHadEvents = true;
								if (Region.Text == FilterEventType)
								{
									Builder.AddEvent(Region.BeginTime, Region.EndTime, CurDepth, Region.Text);
								}
								return true;
							});

						if (RegionHadEvents) CurDepth++;
					});
			}
			else // generic filter
			{
				//TODO: if (EventFilterPtr->FilterEvent(TimingEvent))
			}
		}
	}

}

//////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FMassInsightsTrack::SearchEvent(
	const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindRegionEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const UE::Mass::Trace::FRegion& InEvent)
	{
		FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, reinterpret_cast<uint64>(InEvent.Text));
	});

	return FoundEvent;
}

//////////////////////////////////////////////////////////////////////////

bool FMassInsightsTrack::FindRegionEvent(const FTimingEventSearchParameters& InParameters,
	TFunctionRef<void(double, double, uint32, const UE::Mass::Trace::FRegion&)> InFoundPredicate) const
{
	using namespace UE::Mass::Trace;

	if (!AnalysisSession.IsValid())
	{
		return false;
	}

	// If the query start time is larger than the end of the session return false.
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession.Get());
		if (InParameters.StartTime > AnalysisSession->GetDurationSeconds())
		{
			return false;
		}
	}

	return TTimingEventSearch<FRegion>::Search(
		InParameters,
		[this](TTimingEventSearch<FRegion>::FContext& InContext)
		{
			const IMassTraceProvider& RegionProvider = ReadMassTraceProvider(*AnalysisSession);
			TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

			RegionProvider.EnumerateRegions(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](const FRegion& Region)
			{
				InContext.Check(Region.BeginTime, Region.EndTime, Region.Depth, Region);

				if (!InContext.ShouldContinueSearching())
				{
					return false;
				}

				return true;
			});
		},
		TTimingEventSearch<FRegion>::NoMatch);
}

//////////////////////////////////////////////////////////////////////////

void FMassInsightsTrack::SetFilterConfigurator(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfigurator)
{
	if (FilterConfigurator != InFilterConfigurator)
	{
		FilterConfigurator = InFilterConfigurator;
		SetDirtyFlag();
	}
}

//////////////////////////////////////////////////////////////////////////

} // namespace Insights

#undef LOCTEXT_NAMESPACE
