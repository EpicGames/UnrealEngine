// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Insights/ITimingViewExtender.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Templates/SharedPointer.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Mass::Trace
{
struct FRegion;
}

namespace MassInsightsUI
{

class FMassInsightsTrack;

class FMassInsightsViewCommands : public TCommands<FMassInsightsViewCommands>
{
public:
	FMassInsightsViewCommands();
	virtual ~FMassInsightsViewCommands() override;
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideRegionTrack;
};

class FMassInsightsSharedState : public UE::Insights::Timing::ITimingViewExtender, public TSharedFromThis<FMassInsightsSharedState>
{
	friend class FMassInsightsTrack;
	
public:
	virtual ~FMassInsightsSharedState() override = default;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder) override;

	//////////////////////////////////////////////////
	void ShowHideRegionsTrack();
	bool IsRegionsTrackVisible() const {return bShowHideRegionsTrack;};
	
	void BindCommands();

private:
	TSharedPtr<FMassInsightsTrack> MassInsightsTrack;

	bool bShowHideRegionsTrack = true;
};

class FMassInsightsTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FMassInsightsTrack, FTimingEventsTrack)

public:
	FMassInsightsTrack(FMassInsightsSharedState& InSharedState);
	virtual ~FMassInsightsTrack() override;

	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void SetFilterConfigurator(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfigurator) override;

protected:
	bool FindRegionEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const UE::Mass::Trace::FRegion&)> InFoundPredicate) const;

private:
	TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator;
	FMassInsightsSharedState& SharedState;
	TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession;
};

} // namespace MassInsightsUI