// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Math/ColorList.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"


namespace MassInsights
{

class FMassInsightsStyle final : public FSlateStyleSet
{
public:
	FMassInsightsStyle()
		: FSlateStyleSet("MassInsightsStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		Set("MassProfiler.Icon.Small", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_stats_40x", FVector2D(16.0f, 16.0f)));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FMassInsightsStyle& Get()
	{
		static FMassInsightsStyle Inst;
		return Inst;
	}
	
	~FMassInsightsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

#undef IMAGE_BRUSH

} //namespace SlateInsights


