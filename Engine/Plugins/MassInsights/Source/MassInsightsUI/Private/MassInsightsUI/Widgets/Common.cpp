// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.h"

#include "InsightsCore/Common/TimeUtils.h"

namespace MassInsightsUI
{
	FText FTableCellFormatterTimeHMS::FormatValue(const TOptional<UE::Insights::FTableCellValue>& InValue) const
	{
		if (InValue.IsSet())
		{
			const double Value = InValue.GetValue().Double;
			return FText::FromString(UE::Insights::FormatTimeHMS(Value, UE::Insights::FTimeValue::Microsecond));
		}
		return FText();
	}
}
