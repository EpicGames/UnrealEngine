// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"

namespace MassInsightsUI
{
	DECLARE_DELEGATE_OneParam(FOnSelectedArchetype, uint64);

	/**
	 * Formats the value of a table's cell into time HHh MMm SS.SSSSSSs
	 * ie. 2h 10m 32.893027s
	 */
	class FTableCellFormatterTimeHMS : public UE::Insights::FTableCellValueFormatter
	{
		virtual FText FormatValue(const TOptional<UE::Insights::FTableCellValue>& InValue) const override;
	};
}
