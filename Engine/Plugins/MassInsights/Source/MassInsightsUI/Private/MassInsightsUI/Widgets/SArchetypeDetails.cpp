// Copyright Epic Games, Inc. All Rights Reserved.

#include "SArchetypeDetails.h"

#include "Common/ProviderLock.h"
#include "Hash/BuzHash.h"
#include "Insights/IInsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassTraceTypes.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "MassInsightsUI::SArchetypeDetails"

namespace MassInsightsUI
{
	FName NAME_COLUMN_FragmentName = TEXT("FragmentName");
	FName NAME_COLUMN_FragmentType = TEXT("FragmentType");
	
	class SFragmentListRow : public SMultiColumnTableRow< TSharedPtr<SArchetypeDetails::FFragmentListEntry> >
	{
		using Super = SMultiColumnTableRow< TSharedPtr<SArchetypeDetails::FFragmentListEntry> >;
	public:
		SLATE_BEGIN_ARGS(SFragmentListRow) {}
			SLATE_ARGUMENT(const UE::Mass::Trace::FFragmentInfo*, FragmentInfo)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	private:
		const UE::Mass::Trace::FFragmentInfo* FragmentInfo = nullptr;
	};

	void SFragmentListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FragmentInfo = InArgs._FragmentInfo;
		SetEnabled(true);
		SMultiColumnTableRow< TSharedPtr<SArchetypeDetails::FFragmentListEntry> >::Construct(Super::FArguments(), InOwnerTableView);
	}

	TSharedRef<SWidget> SFragmentListRow::GenerateWidgetForColumn(const FName& InColumnName)
	{
		using namespace UE::Mass::Trace;

		if (InColumnName == NAME_COLUMN_FragmentName)
		{
			return SNew( STextBlock )
					.Text(FText::FromString(FragmentInfo->GetName()));
		}
		else if (InColumnName == NAME_COLUMN_FragmentType)
		{
			// Not a real checkbox, but using it 
			return SNew(SButton)
					.Text_Lambda([this]()
					{
						switch (FragmentInfo->Type)
						{
						case EFragmentType::Fragment: return LOCTEXT("Fragment", "Fragment");
						case EFragmentType::Tag: return LOCTEXT("Tag", "Tag");
						case EFragmentType::Shared: return LOCTEXT("Shared", "Shared");
						case EFragmentType::Sparse: return LOCTEXT("Sparse", "Sparse");
						case EFragmentType::ConstShared: return LOCTEXT("ConstShared", "Const Shared");
						case EFragmentType::SparseTag: return LOCTEXT("SparseTag", "Sparse Tag");
						case EFragmentType::Unknown: // Fallthrough
						default:
							return LOCTEXT("Unknown", "Unknown");
						}
					})
				.ForegroundColor_Lambda(
				[this]()
				{
					switch (FragmentInfo->Type)
					{
						case EFragmentType::Fragment: return FColor(10, 120, 180);
						case EFragmentType::Sparse: return FColor(10, 160, 140);
						case EFragmentType::ConstShared: return FColor(140, 80, 180);
						case EFragmentType::SparseTag: return FColor(120, 140, 10);
						case EFragmentType::Tag: // Fallthrough
						case EFragmentType::Shared: // Fallthrough
						case EFragmentType::Unknown: // Fallthrough
						default:
							return FColor(90, 120, 10);
					}
				});
		}
		else
		{
			return SNew( STextBlock )
					.Text(FText::FromString(TEXT("-")));
		}
	}
	
	SArchetypeDetails::SArchetypeDetails()
		: Table(MakeShared<UE::Insights::FTable>())
	{
	}

	void SArchetypeDetails::Construct(const FArguments& InArgs)
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		Session = CurrentSession;
		
		ChildSlot
		[
			SAssignNew(ListView, STreeView<TSharedPtr<FFragmentListEntry>>)
			.SelectionMode(ESelectionMode::Single)
			.TreeItemsSource(&FilteredSortedFragmentList)
			.OnGetChildren(this, &SArchetypeDetails::TreeView_OnGetChildren)
			.OnGenerateRow(this, &SArchetypeDetails::TreeView_OnGenerateRow)
			.HeaderRow
			(
				SAssignNew(HeaderRow, SHeaderRow)
				.Visibility(EVisibility::Visible)
			)
		];

		TArray<TSharedRef<UE::Insights::FTableColumn>> Columns;
		{
			using namespace UE::Insights;
			TSharedRef<UE::Insights::FTableColumn> ColumnRef = MakeShared<FTableColumn>(NAME_COLUMN_FragmentName);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("FragmentName", "Name"));
			Column.SetTitleName(LOCTEXT("FragmentName", "Name"));
			Column.SetDescription(LOCTEXT("FragmentName", "Name"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(206.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Text);
			Columns.Add(ColumnRef);
		}
		{
			using namespace UE::Insights;
			TSharedRef<UE::Insights::FTableColumn> ColumnRef = MakeShared<FTableColumn>(NAME_COLUMN_FragmentType);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("FragmentType", "Type"));
			Column.SetTitleName(LOCTEXT("FragmentType", "Type"));
			Column.SetDescription(LOCTEXT("FragmentType", "Type"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(206.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Text);
			Columns.Add(ColumnRef);
		}

		Table->SetColumns(Columns);
		
		for (const TSharedRef<UE::Insights::FTableColumn>& ColumnRef : Table->GetColumns())
		{
			if (ColumnRef->ShouldBeVisible())
			{
				ShowColumn(ColumnRef->GetId());
			}
		}
	}

	void SArchetypeDetails::SetArchetype(uint64 InArchetypeID)
	{
		if (ArchetypeID != InArchetypeID)
		{
			ArchetypeID = InArchetypeID;
			bInvalidArchetype = false;
			bArchetypeDirty = true;
		}
	}

	TSharedRef<ITableRow> SArchetypeDetails::TreeView_OnGenerateRow(TSharedPtr<FFragmentListEntry> NodePtr, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<ITableRow> TableRow = SNew(SFragmentListRow, OwnerTable).FragmentInfo(NodePtr->Fragment);
		return TableRow;
	}

	void SArchetypeDetails::TreeView_OnGetChildren(TSharedPtr<FFragmentListEntry> InParent, TArray<TSharedPtr<FFragmentListEntry>>& OutChildren)
	{
	}

	void SArchetypeDetails::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		using namespace UE::Mass::Trace;
		if (bArchetypeDirty
			&& Session.IsValid())
		{
			bArchetypeDirty = false;
			bSortingDirty = true;
			FragmentListData.Reset();

			const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);
			{
				TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);

				const FArchetypeInfo* ArchetypeInfo = Provider.FindArchetypeById(ArchetypeID);
				if (ArchetypeInfo != nullptr)
				{
					for (int32 Index = 0; Index < ArchetypeInfo->Fragments.Num(); Index++)
					{
						FragmentListData.Emplace(MakeShared<FFragmentListEntry>(
							FFragmentListEntry
							{
								.Fragment = ArchetypeInfo->Fragments[Index]
							}));
					}
				}
			}
		}

		if (bSortingDirty)
		{
			bSortingDirty = false;
			FilteredSortedFragmentList.Reset();
			
			// Sort by Type then Name
			FilteredSortedFragmentList.Append(FragmentListData);
			Algo::Sort(FilteredSortedFragmentList, [](const TSharedPtr<FFragmentListEntry>& Lhs, const TSharedPtr<FFragmentListEntry>& Rhs)
			{
				if (Lhs->Fragment->Type == Rhs->Fragment->Type)
				{
					return Lhs->Fragment->Name < Rhs->Fragment->Name;
				}
				return Lhs->Fragment->Type < Rhs->Fragment->Type;
			});

			ListView.Pin()->RebuildList();
			
		}
	}

	void SArchetypeDetails::ShowColumn(const FName ColumnId)
	{
		UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		Column.Show();

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
			.ColumnId(Column.GetId())
			.DefaultLabel(Column.GetShortName())
			.HAlignHeader(Column.GetHorizontalAlignment())
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Fill)
			.VAlignCell(VAlign_Fill)
			.InitialSortMode(Column.GetInitialSortMode())
			.FillWidth(Column.GetInitialWidth())
			.HeaderContent()
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.Padding(FMargin(0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this, ColumnID = Column.GetId()]()->FText
					{
						const UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnID);
						return Column.GetShortName();
					})
				]
			];

		int32 ColumnIndex = 0;
		const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
		const int32 NumColumns = HeaderRow->GetColumns().Num();
		for (; ColumnIndex < NumColumns; ColumnIndex++)
		{
			const SHeaderRow::FColumn& CurrentColumn = HeaderRow->GetColumns()[ColumnIndex];
			const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
			if (NewColumnPosition < CurrentColumnPosition)
			{
				break;
			}
		}

		HeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
	}
}

#undef LOCTEXT_NAMESPACE