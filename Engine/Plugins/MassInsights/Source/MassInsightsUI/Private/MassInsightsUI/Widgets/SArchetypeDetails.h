// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

namespace UE::Mass::Trace
{
struct FFragmentInfo;
}

namespace UE::Insights
{
	class FTable;
}

namespace TraceServices
{
	class IAnalysisSession;
}

namespace MassInsightsUI
{
	class SArchetypeDetails : public SCompoundWidget
	{
	public:
		SArchetypeDetails();
		SLATE_BEGIN_ARGS(SArchetypeDetails) {}
			
		SLATE_END_ARGS()

		void Construct(const FArguments& _args);
		void SetArchetype(uint64 ArchetypeID);
	private:
		friend class SFragmentListRow;
		struct FFragmentListEntry
		{
			const UE::Mass::Trace::FFragmentInfo* Fragment = nullptr;
		};

		TSharedRef<ITableRow> TreeView_OnGenerateRow(TSharedPtr<FFragmentListEntry> NodePtr, const TSharedRef<STableViewBase>& OwnerTable);
		void TreeView_OnGetChildren(TSharedPtr<FFragmentListEntry> InParent, TArray<TSharedPtr<FFragmentListEntry>>& OutChildren);
		
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		void ShowColumn(const FName ColumnId);

		TSharedPtr<UE::Insights::FTable> Table;
		uint64 ArchetypeID = 0;
		bool bArchetypeDirty = true;
		bool bSortingDirty = true;
		bool bInvalidArchetype = true;
		TSharedPtr<const TraceServices::IAnalysisSession> Session;
		TWeakPtr<STreeView<TSharedPtr<FFragmentListEntry>>> ListView;
		TSharedPtr<SHeaderRow> HeaderRow;
		TArray<TSharedPtr<FFragmentListEntry>> FragmentListData;
		TArray<TSharedPtr<FFragmentListEntry>> FilteredSortedFragmentList;
	};
}
