// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEntityEventAggregationTableView.h"

#include "Algo/ForEach.h"
#include "Common/ProviderLock.h"
#include "Insights/IInsightsManager.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassTraceTypes.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MassInsights::SEntityTimelineTableView"

namespace MassInsights
{
	static FName COLUMN_EntityID = FName("EntityID");
	static FName COLUMN_Archetype = FName("Archetype");
	static FName COLUMN_CreateTime = FName("CreateTime");
	static FName COLUMN_EventCount = FName("EventCount");
	static FName COLUMN_LastEventTime = FName("LastEventTime");
	static FName COLUMN_Alive = FName("Alive");
	
	INSIGHTS_IMPLEMENT_RTTI(FEntityEventAggregateRecordHandle)
	
	FEntityEventAggregateRecordHandle::FEntityEventAggregateRecordHandle(int32 InRecordIndex)
		: FBaseTreeNode(false /* bInIsGroup */)
		, RecordIndex(InRecordIndex)
	{
	}

	const FText FEntityEventAggregateRecordHandle::GetDisplayName() const
	{
		return FText::FromString(FString::Printf(TEXT("record_%d"), RecordIndex));
	}

	int32 FEntityEventAggregateRecordHandle::GetRecordIndex() const
	{
		return RecordIndex;
	}

	// Helper template to read the value referenced by a data member pointer from FEntityEventAggregationNode and convert it to
	// a FTableCellValue
	template<typename MemberPtr>
	class SEntityEventAggregationTableView::TAggregationRecordMemberAccessor : public UE::Insights::FTableCellValueGetter
	{
	public:
		using FTableCellValue = UE::Insights::FTableCellValue;
		using FTableColumn = UE::Insights::FTableColumn;
		using FBaseTreeNode = UE::Insights::FBaseTreeNode;
		
		explicit TAggregationRecordMemberAccessor(MemberPtr InPtr, const TArray<FEntityEventAggregationNode>& InEventAggregationRecords)
			: Ptr(InPtr)
			, EventAggregationRecords(InEventAggregationRecords)
		{}
				
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			const int32 Index = static_cast<const FEntityEventAggregateRecordHandle&>(Node).GetRecordIndex();
			const SEntityEventAggregationTableView::FEntityEventAggregationNode& AggregationRecord = EventAggregationRecords[Index];
			return FTableCellValue(AggregationRecord.*Ptr);
		}
		MemberPtr Ptr;
		const TArray<FEntityEventAggregationNode>& EventAggregationRecords;
	};
	
	template <typename MemberPtr>
	constexpr auto SEntityEventAggregationTableView::MakeAccessor(MemberPtr Ptr)
	{
		return MakeShareable(new TAggregationRecordMemberAccessor<MemberPtr>(Ptr, EventAggregationRecords));
	}

	void SEntityEventAggregationTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		TablePtr = InArgs._TablePtr;
		RecordHandle = InArgs._RowHandle;
		AggregationTableView = InArgs._ViewModel;
		OnArchetypeSelected = InArgs._OnArchetypeSelected;
		SetEnabled(true);
		Super::Construct(Super::FArguments(), InOwnerTableView);
	}

	TSharedRef<SWidget> SEntityEventAggregationTableRow::GenerateWidgetForColumn(const FName& InColumnName)
	{
		int32 Index = RecordHandle->GetRecordIndex();
		TSharedPtr<SEntityEventAggregationTableView> VM = AggregationTableView.Pin();
		const SEntityEventAggregationTableView::FEntityEventAggregationNode& AggregationRecord = VM->EventAggregationRecords[Index];

		TSharedPtr<UE::Insights::FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(InColumnName);
		if (!ColumnPtr.IsValid())
		{
			return SNew( STextBlock )
				.Text(LOCTEXT("Unknown Column Message", "Unknown Column"));
		}
		
		if (InColumnName == COLUMN_EntityID)
		{
			return SNew( STextBlock )
				.Text(FText::Format(LOCTEXT("EntityID_Cell", "{0}"), AggregationRecord.EntityID ));
		}
		else if (InColumnName == COLUMN_Archetype)
		{
			return SNew(SButton)
				.Text(LOCTEXT("Archetype", "Archetype"))
				.OnReleased(this, &SEntityEventAggregationTableRow::OnReleasedArchetypeButton, AggregationRecord.LastArchetype);
		}
		else if (InColumnName == COLUMN_CreateTime)
		{
			return SNew( STextBlock )
				.Text(ColumnPtr->GetValueAsText(*RecordHandle));
		}
		else if (InColumnName == COLUMN_EventCount)
		{
			return SNew( STextBlock )
				.Text(FText::Format(LOCTEXT("EventCount", "{0}"), AggregationRecord.Events ));
		}
		else if (InColumnName == COLUMN_LastEventTime)
		{
			return SNew( STextBlock )
				.Text(ColumnPtr->GetValueAsText(*RecordHandle));
		}
		else if (InColumnName == COLUMN_Alive)
		{
			if (AggregationRecord.bDestroyed)
			{
				return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.X"))
					.DesiredSizeOverride(FVector2D(16.0, 16.0))
					.ColorAndOpacity(FSlateColor(FColor(160.0f, 20.0f, 30.0f)))
				];
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		}
		else
		{
			return SNew( STextBlock )
				.Text(LOCTEXT("Unknown Column", "Unknown Column"));
		}
	}

	void SEntityEventAggregationTableRow::OnReleasedArchetypeButton(uint64 ArchetypeID)
	{
		OnArchetypeSelected.ExecuteIfBound(ArchetypeID);
	}


	SEntityEventAggregationTableView::SEntityEventAggregationTableView()
		: Table(MakeShared<UE::Insights::FTable>())
	{
	}

	SEntityEventAggregationTableView::~SEntityEventAggregationTableView()
	{
	}

	void SEntityEventAggregationTableView::Construct(const FArguments& InArgs)
	{
		OnArchetypeSelected = InArgs._OnArchetypeSelected;
		OnRowSelected = InArgs._OnRowSelected;
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				// TODO: This is a good place to add filtering tools
				SNullWidget::NullWidget
			]
			
			+SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f)
				[
					SAssignNew(TreeView, STreeView<TSharedPtr<FEntityEventAggregateRecordHandle>>)
					.SelectionMode(ESelectionMode::Single)
					.TreeItemsSource(&FilteredTreeViewRows)
					.OnGetChildren(this, &SEntityEventAggregationTableView::TreeView_OnGetChildren)
					.OnGenerateRow(this, &SEntityEventAggregationTableView::TreeView_OnGenerateRow)
					.OnSelectionChanged_Lambda([this](TSharedPtr<FEntityEventAggregateRecordHandle> Item, ESelectInfo::Type SelectInfo)
					{
						TArray<TSharedPtr<FEntityEventAggregateRecordHandle>> SelectedItems = TreeView->GetSelectedItems();
						if (SelectedItems.Num() == 0)
						{
							FEntityEventSummaryRowSelectedParams Params
							{
								.IsSelected = false
							};
							this->OnRowSelected.ExecuteIfBound(Params);
						}
						else if (EventAggregationRecords.IsValidIndex(SelectedItems[0]->GetRecordIndex()))
						{
							const FEntityEventAggregationNode& EventAggregationNode = EventAggregationRecords[SelectedItems[0]->GetRecordIndex()];
							FEntityEventSummaryRowSelectedParams Params
							{
								.IsSelected = true,
								.EntityID = EventAggregationNode.EntityID,
								.FirstEventTime = EventAggregationNode.FirstEventTime,
								.LastEventTime = EventAggregationNode.LastEventTime,
								.TotalEvents = EventAggregationNode.Events
							};
							this->OnRowSelected.ExecuteIfBound(Params);
						}
					})
					
					.HeaderRow
					(
						SAssignNew(HeaderRow, SHeaderRow)
						.Visibility(EVisibility::Visible)
					)
				]
			]

			// Status bar
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			.Padding(0.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FLinearColor(0.05f, 0.1f, 0.2f, 1.0f))
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Margin(FMargin(4.0f, 1.0f, 4.0f, 1.0f))
					.Text_Raw(this, &SEntityEventAggregationTableView::GetStatusBarText)
					.ColorAndOpacity(FLinearColor(1.0f, 0.75f, 0.5f, 1.0f))
					.Visibility(EVisibility::Visible)
				]
			]
		];

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
		if (InsightsManager.IsValid())
		{
			InsightsManager->GetSessionChangedEvent().AddSP(this, &SEntityEventAggregationTableView::InsightsManager_OnSessionChanged);
			InsightsManager->GetSessionAnalysisCompletedEvent().AddSP(this, &SEntityEventAggregationTableView::InsightsManager_OnSessionAnalysisComplete);
		}
		
		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		Session = CurrentSession;

		TArray<TSharedRef<UE::Insights::FTableColumn>> Columns;		
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_CreateTime);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("First Event Time", "First Event Time"));
			Column.SetTitleName(LOCTEXT("First Event Time", "First Event Time"));
			Column.SetDescription(LOCTEXT("First Event Time", "First Event Time"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(100.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Double);
			
			TSharedRef<ITableCellValueGetter> Getter = MakeAccessor(&FEntityEventAggregationNode::FirstEventTime);//MakeShared<FDisplayNameValueGetter>();
			Column.SetValueGetter(Getter);
			
			Column.SetValueFormatter(MakeShared<MassInsightsUI::FTableCellFormatterTimeHMS>());

			TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
			Column.SetValueSorter(Sorter);
			
			Columns.Add(ColumnRef);
		}
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_EntityID);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("EntityID", "EntityID"));
			Column.SetTitleName(LOCTEXT("EntityID", "EntityID"));
			Column.SetDescription(LOCTEXT("EntityID", "EntityID"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(100.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Custom);
			
			Columns.Add(ColumnRef);
		}
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_EventCount);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("Total Events", "Total Events"));
			Column.SetTitleName(LOCTEXT("Total Events", "Total Events"));
			Column.SetDescription(LOCTEXT("Total Events", "Total Events"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(80.0f);
			Column.SetMinWidth(41.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Int64);
			
			Columns.Add(ColumnRef);
		}
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_Alive);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("Status", "Status"));
			Column.SetTitleName(LOCTEXT("Status", "Status"));
			Column.SetDescription(LOCTEXT("Status", "Status"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(30.0f);
			Column.SetMinWidth(30.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Int64);
			
			Columns.Add(ColumnRef);
		}
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_LastEventTime);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("Last Event Time", "Last Event Time"));
			Column.SetTitleName(LOCTEXT("Last Event Time", "Last Event Time"));
			Column.SetDescription(LOCTEXT("Last Event Time", "Last Event Time"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(100.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Custom);
			
			TSharedRef<ITableCellValueGetter> Getter = MakeAccessor(&FEntityEventAggregationNode::LastEventTime);//MakeShared<FDisplayNameValueGetter>();
			Column.SetValueGetter(Getter);
			
			TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<MassInsightsUI::FTableCellFormatterTimeHMS>();
			Column.SetValueFormatter(Formatter);
			
			Columns.Add(ColumnRef);
		}
		
		Table->SetColumns(Columns);

		for (const TSharedRef<UE::Insights::FTableColumn>& ColumnRef : Table->GetColumns())
		{
			if (ColumnRef->ShouldBeVisible())
			{
				ShowColumn(ColumnRef->GetId());
			}
		}
	}

	void SEntityEventAggregationTableView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		
		if (bRebuildTree)
		{
			EntityToRecordIndexMap.Empty();
			EventsProcessed = 0;
			EventAggregationRecords.Empty();
			TreeView->RebuildList();
			
			bRebuildTree = false;
		}

		IncrementalUpdate();
	}

	void SEntityEventAggregationTableView::InsightsManager_OnSessionChanged()
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		
		Session = CurrentSession;
		Reset();
	}

	void SEntityEventAggregationTableView::InsightsManager_OnSessionAnalysisComplete()
	{
	}

	TSharedRef<ITableRow> SEntityEventAggregationTableView::TreeView_OnGenerateRow(TSharedPtr<FEntityEventAggregateRecordHandle> RowHandle,
	                                                                 const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SWidget> SharedThisAsWidget = AsShared();
		auto SharedThis = StaticCastSharedPtr<SEntityEventAggregationTableView>(SharedThisAsWidget);
		TSharedRef<ITableRow> TableRow =
			SNew(MassInsights::SEntityEventAggregationTableRow, OwnerTable)
			.TablePtr(Table)
			.RowHandle(RowHandle)
			.ViewModel(SharedThis)
			.OnArchetypeSelected_Lambda([this](uint64 ArchetypeID)
			{
				OnArchetypeSelected.ExecuteIfBound(ArchetypeID);
			});
	
		return TableRow;
	}

	void SEntityEventAggregationTableView::TreeView_OnGetChildren(TSharedPtr<FEntityEventAggregateRecordHandle> InParent,
		TArray<TSharedPtr<FEntityEventAggregateRecordHandle>>& OutChildren)
	{
		return;
	}

	void SEntityEventAggregationTableView::Reset()
	{
		bRebuildTree = true;
	}

	void SEntityEventAggregationTableView::IncrementalUpdate()
	{
		using namespace UE::Mass::Trace;

		if (Session.IsValid())
		{
			uint64 PostUpdate_NextIncrementalEventIndex = EventsProcessed;

			const int32 PreUpdateRecordCount = EventAggregationRecords.Num();

			// @TODO: Would be better if this were async instead of on Tick
			const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);
			{
				TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);

				constexpr uint64 MaximumEventsToCheck = 100000;

				EstimatedEventCount = Provider.GetEntityEventCount();
				
				Provider.EnumerateEntityEvents(
					EventsProcessed,
					MaximumEventsToCheck,
					[this, &PostUpdate_NextIncrementalEventIndex](const FEntityEventRecord& Event, uint64 EventIndex)
					{
						++PostUpdate_NextIncrementalEventIndex;
						
						switch (Event.Operation)
						{
						case EEntityEventType::Created:
							{
								const int32 RecordIndex = EventAggregationRecords.Num();
								EntityToRecordIndexMap.Add(Event.Entity, EventAggregationRecords.Num());
								EventAggregationRecords.Emplace(FEntityEventAggregationNode
									{
										.EntityID = Event.Entity,
										.LastArchetype = Event.ArchetypeID,
										.Events = 1,
										.bDestroyed = false,
										.FirstEventTime = Event.Time,
										.LastEventTime = Event.Time,
										.RowHandle = MakeShared<FEntityEventAggregateRecordHandle>(RecordIndex)
									});
							}
							break;
						case EEntityEventType::ArchetypeChange:
							{
								int32* IndexPtr = EntityToRecordIndexMap.Find(Event.Entity);
								if (IndexPtr != nullptr)
								{
									// Update the row, maintain the creation time		
									FEntityEventAggregationNode& RecordIndex = EventAggregationRecords[*IndexPtr];
									RecordIndex.LastArchetype = Event.ArchetypeID;
									RecordIndex.Events++;
									RecordIndex.LastEventTime = Event.Time;
								}
								else
								{
									const int32 RecordIndex = EventAggregationRecords.Num();
									EntityToRecordIndexMap.Add(Event.Entity, EventAggregationRecords.Num());
									EventAggregationRecords.Emplace(FEntityEventAggregationNode
									{
										.EntityID = Event.Entity,
										.LastArchetype = Event.ArchetypeID,
										.Events = 1,
										.bDestroyed = false,
										.FirstEventTime = Event.Time,
										.LastEventTime = Event.Time,
										.RowHandle = MakeShared<FEntityEventAggregateRecordHandle>(RecordIndex)
									});
								}							
							}
							break;
						case EEntityEventType::Destroyed:
							{
								int32* IndexPtr = EntityToRecordIndexMap.Find(Event.Entity);
								if (IndexPtr != nullptr)
								{
									FEntityEventAggregationNode& Record = EventAggregationRecords[*IndexPtr];
									Record.Events++;
									Record.bDestroyed = true;
									Record.LastEventTime = Event.Time;
								}
								else
								{
									const int32 RecordIndex = EventAggregationRecords.Num();
									EntityToRecordIndexMap.Add(Event.Entity, EventAggregationRecords.Num());
									EventAggregationRecords.Emplace(FEntityEventAggregationNode
									{
										.EntityID = Event.Entity,
										.LastArchetype = Event.ArchetypeID,
										.Events = 1,
										.bDestroyed = true,
										.FirstEventTime = Event.Time,
										.LastEventTime = Event.Time,
										.RowHandle = MakeShared<FEntityEventAggregateRecordHandle>(RecordIndex)
									});
								}
							}
							break;
						}
					});
			}
			
			if (EventsProcessed != PostUpdate_NextIncrementalEventIndex)
			{
				for (int32 RecordIndex = PreUpdateRecordCount; RecordIndex < EventAggregationRecords.Num(); RecordIndex++)
				{
					FilteredTreeViewRows.Add(EventAggregationRecords[RecordIndex].RowHandle);
				}
				
				TreeView->RequestListRefresh();
			}
			EventsProcessed = PostUpdate_NextIncrementalEventIndex;
		};
	}

	void SEntityEventAggregationTableView::ShowColumn(const FName ColumnId)
	{
		UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		Column.Show();

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
			.ColumnId(Column.GetId())
			.DefaultLabel(Column.GetShortName())
			.HAlignHeader(Column.GetHorizontalAlignment())
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Fill)
			.VAlignCell(VAlign_Fill)
			.InitialSortMode(Column.GetInitialSortMode())
			.FillWidth(Column.GetInitialWidth())
			.HeaderContent()
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.Padding(FMargin(0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SEntityEventAggregationTableView::GetColumnHeaderText, Column.GetId())
				]
			];

		int32 ColumnIndex = 0;
		const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
		const int32 NumColumns = HeaderRow->GetColumns().Num();
		for (; ColumnIndex < NumColumns; ColumnIndex++)
		{
			const SHeaderRow::FColumn& CurrentColumn = HeaderRow->GetColumns()[ColumnIndex];
			const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
			if (NewColumnPosition < CurrentColumnPosition)
			{
				break;
			}
		}

		HeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
	}

	FText SEntityEventAggregationTableView::GetColumnHeaderText(const FName ColumnId) const
	{
		const UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		return Column.GetShortName();
	}

	FText SEntityEventAggregationTableView::GetStatusBarText() const
	{
		return FText::Format(LOCTEXT("EntityStatusBarText", "-- Entities:{0} Events:{1}/{2}--"), FilteredTreeViewRows.Num(), EventsProcessed, EstimatedEventCount);
	}

}

#undef LOCTEXT_NAMESPACE