// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SFragmentTableRow.h"
#include "Framework/Commands/UICommandList.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "MassInsightsUI/Widgets/Common.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace MassInsights
{
	struct FEntityEventAggregateRecordHandle : public UE::Insights::FBaseTreeNode
	{
		INSIGHTS_DECLARE_RTTI(FEntityEventAggregateRecordHandle, UE::Insights::FBaseTreeNode);

	public:
		explicit FEntityEventAggregateRecordHandle(int32 InRecordIndex);

		virtual const FText GetDisplayName() const override;
		int32 GetRecordIndex() const;

	private:
		// Index used to reference event aggregation data
		int32 RecordIndex;
	};

	class SEntityEventAggregationTableView;
	
	class SEntityEventAggregationTableRow : public SMultiColumnTableRow< TSharedPtr<FEntityEventAggregateRecordHandle> >
	{
		using Super = SMultiColumnTableRow< TSharedPtr<FEntityEventAggregateRecordHandle> >;
	public:
		SLATE_BEGIN_ARGS(SEntityEventAggregationTableRow) {}
			SLATE_ARGUMENT(TSharedPtr<UE::Insights::FTable>, TablePtr)
			SLATE_ARGUMENT(TSharedPtr<FEntityEventAggregateRecordHandle>, RowHandle)
			SLATE_ARGUMENT(TSharedPtr<SEntityEventAggregationTableView>, ViewModel)
			SLATE_EVENT(MassInsightsUI::FOnSelectedArchetype, OnArchetypeSelected)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	private:
		void OnReleasedArchetypeButton(uint64 ArchetypeID);
		TSharedPtr<UE::Insights::FTable> TablePtr;
		TWeakPtr<SEntityEventAggregationTableView> AggregationTableView;
		TSharedPtr<FEntityEventAggregateRecordHandle> RecordHandle;
		MassInsightsUI::FOnSelectedArchetype OnArchetypeSelected;
		
	};

	// Struct to make it easier to extend passing parameters to this delegate
	struct FEntityEventSummaryRowSelectedParams
	{
		// False if deselected, all fields are undefined on deselection
		// True if selected, all fields should describe the row selected
		bool IsSelected;
		uint64 EntityID;
		double FirstEventTime;
		double LastEventTime;
		int64 TotalEvents;
	};
	DECLARE_DELEGATE_OneParam(FEntityEventContainerRowSelected, const FEntityEventSummaryRowSelectedParams&);
	
	class SEntityEventAggregationTableView : public SCompoundWidget
	{
		friend class SEntityEventAggregationTableRow;
	public:
		SEntityEventAggregationTableView();
		virtual ~SEntityEventAggregationTableView();

		SLATE_BEGIN_ARGS(SEntityEventAggregationTableView) {}
			SLATE_EVENT(MassInsightsUI::FOnSelectedArchetype, OnArchetypeSelected)
			SLATE_EVENT(FEntityEventContainerRowSelected, OnRowSelected)
		SLATE_END_ARGS()

		void Construct(const FArguments& args);
		void Reset();
	private:
		void InsightsManager_OnSessionChanged();
		void InsightsManager_OnSessionAnalysisComplete();
		
		TSharedRef<ITableRow> TreeView_OnGenerateRow(TSharedPtr<FEntityEventAggregateRecordHandle> NodePtr, const TSharedRef<STableViewBase>& OwnerTable);
		void TreeView_OnGetChildren(TSharedPtr<FEntityEventAggregateRecordHandle> InParent, TArray<TSharedPtr<FEntityEventAggregateRecordHandle>>& OutChildren);
	
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		
		void IncrementalUpdate();
		void ShowColumn(const FName ColumnId);
		FText GetColumnHeaderText(const FName ColumnId) const;

		FText GetStatusBarText() const;
		
	private:
		TSharedPtr<UE::Insights::FTable> Table;
		TSharedPtr<const TraceServices::IAnalysisSession> Session;
		TSharedPtr<FUICommandList> CommandList;
		
		TSharedPtr<STreeView<TSharedPtr<FEntityEventAggregateRecordHandle>>> TreeView;
		TSharedPtr<SHeaderRow> HeaderRow;
		
		// Collated summary data about the events gathered from the application
		struct FEntityEventAggregationNode
		{
			uint64 EntityID;
			// ID of the last archetype this entity had
			uint64 LastArchetype;
			int32 Events = 0;
			bool bDestroyed = false;
			double FirstEventTime;
			double LastEventTime;
			TSharedPtr<FEntityEventAggregateRecordHandle> RowHandle;
		};

		template<typename MemberPtr>
		class TAggregationRecordMemberAccessor;

		template<typename MemberPtr>
		constexpr auto MakeAccessor(MemberPtr Ptr);
		
		TArray<FEntityEventAggregationNode> EventAggregationRecords;

		TMap<uint64, int32> EntityToRecordIndexMap;

		uint64 EventsProcessed = 0;
		uint64 EstimatedEventCount = 0;

		bool bRebuildTree = true;
		
		TArray<TSharedPtr<FEntityEventAggregateRecordHandle>> FilteredTreeViewRows;
		
		MassInsightsUI::FOnSelectedArchetype OnArchetypeSelected;
		FEntityEventContainerRowSelected OnRowSelected;
	};
}
