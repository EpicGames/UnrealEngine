// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEntityEventsTableView.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Algo/LevenshteinDistance.h"
#include "Common/ProviderLock.h"
#include "Insights/IInsightsManager.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "MassInsightsUI/Widgets/Common.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassTraceTypes.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MassInsights::SEntityTimelineTableView"

namespace MassInsightsUI
{
	static FName COLUMN_EventTime = TEXT("EventTime");
	static FName COLUMN_Operation = TEXT("Operation");

	class SEntityEventsTableRow : public SMultiColumnTableRow< TSharedPtr<SEntityEventsTableView::FEventNode> >
	{
		using Super = SMultiColumnTableRow< TSharedPtr<SEntityEventsTableView::FEventNode> >;
	public:
		SLATE_BEGIN_ARGS(SEntityEventsTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<UE::Insights::FTable>, TablePtr)
		SLATE_ARGUMENT(TSharedPtr<SEntityEventsTableView::FEventNode>, EventNode)
		SLATE_ARGUMENT(TSharedPtr<SEntityEventsTableView>, ViewModel)
		SLATE_ARGUMENT(TSharedPtr<const TraceServices::IAnalysisSession>, AnalysisSession)
		SLATE_EVENT(FOnSelectedArchetype, OnArchetypeSelected)
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	private:
		void OnReleasedArchetypeButton(uint64 ArchetypeID);
		TSharedPtr<UE::Insights::FTable> TablePtr;
		TWeakPtr<SEntityEventsTableView> ViewModel;
		TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession;
		TSharedPtr<SEntityEventsTableView::FEventNode> EventNode;
		FOnSelectedArchetype OnArchetypeSelected;
		
	};

	void SEntityEventsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		TablePtr = InArgs._TablePtr;
		EventNode = InArgs._EventNode;
		ViewModel = InArgs._ViewModel;
		AnalysisSession = InArgs._AnalysisSession;
		check(TablePtr);
		check(EventNode);
		check(ViewModel.IsValid());
		check(AnalysisSession);
		OnArchetypeSelected = InArgs._OnArchetypeSelected;
		SetEnabled(true);
		Super::Construct(Super::FArguments(), InOwnerTableView);
	}
	
	TSharedRef<SWidget> SEntityEventsTableRow::GenerateWidgetForColumn(const FName& InColumnName)
	{
		using namespace UE::Mass::Trace;

		const uint64 ProviderEventIndex = EventNode->GetProviderEventIndex();
		const IMassTraceProvider& Provider = ReadMassTraceProvider(*AnalysisSession);

		TValueOrError<FEntityEventRecord, void> Event = MakeError();
		{
			TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);

			Event = Provider.GetEntityEvent(ProviderEventIndex);
		}
		if (!Event.HasValue())
		{
			return SNew(STextBlock).Text(LOCTEXT("MissingEventIndex", "N/A'"));
		}
		
		TSharedPtr<UE::Insights::FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(InColumnName);

		if (InColumnName == COLUMN_EventTime)
		{
			TOptional<UE::Insights::FTableCellValue> ValueThing = ColumnPtr->GetValue(*EventNode);
			return SNew( STextBlock )
				.Text(ColumnPtr->GetValueAsText(*EventNode));
		}
		else if (InColumnName == COLUMN_Operation)
		{
			const FSlateBrush* Icon = [EventType = Event.GetValue().Operation]
			{
				switch (EventType)
				{
				case EEntityEventType::Created:
					return FAppStyle::GetBrush("Icons.Plus");
				case EEntityEventType::ArchetypeChange:
					return FAppStyle::GetBrush("Icons.ArrowRight");
				case EEntityEventType::Destroyed:
					return FAppStyle::GetBrush("Icons.X");
				default:
					return FAppStyle::GetBrush("Icons.Warning");
				}
			}();
			
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(Icon)
					.DesiredSizeOverride(FVector2D(16.0, 16.0))
					.ColorAndOpacity([EventType = Event.GetValue().Operation]()
					{
						switch (EventType)
						{
						case EEntityEventType::Created:
							return FSlateColor(FColor(30.0f, 200.0f, 20.0f));
						case EEntityEventType::ArchetypeChange:
							return FSlateColor(FColor(200.0f, 200.0f, 200.0f));
						case EEntityEventType::Destroyed:
							return FSlateColor(FColor(160.0f, 20.0f, 30.0f));
						default:
							return FSlateColor(FColor(255,255,255));
						}
					}())
				];
		}
		
		return SNew( STextBlock )
				.Text(LOCTEXT("Unknown Column", "Unknown Column"));
	}

	void SEntityEventsTableRow::OnReleasedArchetypeButton(uint64 ArchetypeID)
	{
		OnArchetypeSelected.ExecuteIfBound(ArchetypeID);
	}
	
	SEntityEventsTableView::FEventNode::FEventNode(uint64 InProviderEntityIndex)
		: FBaseTreeNode(false /* bInIsGroup */)
		, ProviderEntityIndex(InProviderEntityIndex)
	{
	}

	const FText SEntityEventsTableView::FEventNode::GetDisplayName() const
	{
		return FText::FromString(FString::Printf(TEXT("event_%llu"), ProviderEntityIndex));
	}

	INSIGHTS_IMPLEMENT_RTTI(SEntityEventsTableView::FEventNode);

	uint64 SEntityEventsTableView::FEventNode::GetProviderEventIndex() const
	{
		return ProviderEntityIndex;
	}

	SEntityEventsTableView::SEntityEventsTableView()
		: Table(MakeShared<UE::Insights::FTable>())
	{
	}

	void SEntityEventsTableView::TreeView_OnSelectionChanged(TSharedPtr<FEventNode> EventNode, ESelectInfo::Type Arg) const
	{
		if (EventNode)
		{
			FOnSelectedEntityEventParams Params;
			Params.ProviderEventIndex = EventNode->GetProviderEventIndex();
			
			OnEntityEventSelected.ExecuteIfBound(Params);
		}
	}

	void SEntityEventsTableView::Construct(const FArguments& InArgs)
	{
		using namespace UE::Mass::Trace;

		OnEntityEventSelected = InArgs._OnEntityEventSelected;
		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				// TODO Filtering stuff here
				SNullWidget::NullWidget
			]
					
			+SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				SNew(SHorizontalBox)
						
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f)
				[
					SAssignNew(TreeView, STreeView<TSharedPtr<FEventNode>>)
					.SelectionMode(ESelectionMode::Single)
					.TreeItemsSource(&FilteredSortedEvents)
					.OnGetChildren(this, &SEntityEventsTableView::TreeView_OnGetChildren)
					.OnGenerateRow(this, &SEntityEventsTableView::TreeView_OnGenerateRow)
					.OnSelectionChanged(this, &SEntityEventsTableView::TreeView_OnSelectionChanged)
							
					.HeaderRow
					(
						SAssignNew(HeaderRow, SHeaderRow)
						.Visibility(EVisibility::Visible)
					)
				]
			]
		];

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
		if (InsightsManager.IsValid())
		{
			InsightsManager->GetSessionChangedEvent().AddSP(this, &SEntityEventsTableView::InsightsManager_OnSessionChanged);
		}
		
		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		Session = CurrentSession;

		TArray<TSharedRef<UE::Insights::FTableColumn>> Columns;
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_EventTime);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("Event Time", "Event Time"));
			Column.SetTitleName(LOCTEXT("Event Time", "Event Time"));
			Column.SetDescription(LOCTEXT("Event Time", "Event Time"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(206.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Double);

			class FEventTimeGetter : public FTableCellValueGetter
			{
			public:
				virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
				{
					const FEventNode& EventNode = static_cast<const FEventNode&>(Node);
					const uint64 ProviderEventIndex = EventNode.GetProviderEventIndex();

					IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
					TSharedPtr<Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
					if (InsightsManager.IsValid())
					{
						TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
						if (CurrentSession.IsValid())
						{
							const IMassTraceProvider& Provider = ReadMassTraceProvider(*CurrentSession);
			
							{
								TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);
								TValueOrError<FEntityEventRecord, void> EventData = Provider.GetEntityEvent(ProviderEventIndex);
								if (EventData.HasValue())
								{
									return FTableCellValue(EventData.GetValue().Time);
								}
							}
						}
					}
					return TOptional<FTableCellValue>();
				}
			};

			Column.SetValueGetter(MakeShared<FEventTimeGetter>());

			Column.SetValueFormatter(MakeShared<FTableCellFormatterTimeHMS>());
			
			Columns.Add(ColumnRef);
		}
		
		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(COLUMN_Operation);
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("Operation", "Operation"));
			Column.SetTitleName(LOCTEXT("Operation", "Operation"));
			Column.SetDescription(LOCTEXT("Operation", "Operation"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(206.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Int64);
			
			Columns.Add(ColumnRef);
		}
		
		Table->SetColumns(Columns);

		for (const TSharedRef<UE::Insights::FTableColumn>& ColumnRef : Table->GetColumns())
		{
			if (ColumnRef->ShouldBeVisible())
			{
				ShowColumn(ColumnRef->GetId());
			}
		}
	}

	void SEntityEventsTableView::SetEntities(TConstArrayView<uint64> InEntities)
	{
		Entities = InEntities;
		bRebuildTable = true;
	}

	TSharedRef<ITableRow> SEntityEventsTableView::TreeView_OnGenerateRow(
		TSharedPtr<FEventNode> InEventNode,
		const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SWidget> SharedThisAsWidget = AsShared();
		auto SharedThis = StaticCastSharedPtr<SEntityEventsTableView>(SharedThisAsWidget);
		TSharedRef<ITableRow> TableRow =
			SNew(MassInsightsUI::SEntityEventsTableRow, OwnerTable)
			.TablePtr(Table)
			.EventNode(InEventNode)
			.ViewModel(SharedThis)
			.AnalysisSession(Session)
			.OnArchetypeSelected_Lambda([this](uint64 ArchetypeID)
			{
				OnArchetypeSelected.ExecuteIfBound(ArchetypeID);
			});
	
		return TableRow;
	}

	void SEntityEventsTableView::TreeView_OnGetChildren(TSharedPtr<FEventNode> InParent, TArray<TSharedPtr<FEventNode>>& OutChildren)
	{
	}

	void SEntityEventsTableView::IncrementalUpdate()
	{
		using namespace UE::Mass::Trace;

		if (Session.IsValid())
		{
			const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);

			// Note: Would be better if this were async instead of on Tick
			{
				TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);

				constexpr uint64 MaximumEventsToProcess = 10000;

				const int32 PreviousEventNodeCount = Events.Num();

				uint64 NewNextEventIndex = NextEventIndex;
				Provider.EnumerateEntityEvents(NextEventIndex, MaximumEventsToProcess, [this, &NewNextEventIndex](const FEntityEventRecord& Event, uint64 ProviderEventIndex)
				{
					++NewNextEventIndex;

					const uint64* EntityPtr = Algo::Find(Entities, Event.Entity);
					if (!EntityPtr)
					{
						return;
					}

					const int32 EntityIndex = static_cast<int32>(static_cast<uint64>((EntityPtr - Entities.GetData())) / sizeof(uint64));
					check(EntityIndex >= 0 && EntityIndex < Entities.Num());

					TSharedPtr<FEventNode> EventNode = MakeShared<FEventNode>(ProviderEventIndex);
					Events.Emplace(MoveTemp(EventNode));
				});
				
				const int32 NewEventNodesAdded = Events.Num() - PreviousEventNodeCount;
				
				if (NewEventNodesAdded > 0)
				{
					// Should already be sorted
					FilteredSortedEvents.Reserve(Events.Num());
					for (int32 Index = PreviousEventNodeCount; Index < Events.Num(); Index++)
					{
						FilteredSortedEvents.Add(Events[Index]);
					}
					
					TreeView->RebuildList();
				}
				
				NextEventIndex = NewNextEventIndex;
				
			}
		}
	}

	void SEntityEventsTableView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (bRebuildTable)
		{
			NextEventIndex = 0;
			Events.Reset();
			FilteredSortedEvents.Reset();
			TreeView->RebuildList();
			
			bRebuildTable = false;
		}

		IncrementalUpdate();
	}

	void SEntityEventsTableView::ShowColumn(const FName ColumnId)
	{
		UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		Column.Show();

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
			.ColumnId(Column.GetId())
			.DefaultLabel(Column.GetShortName())
			.HAlignHeader(Column.GetHorizontalAlignment())
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Fill)
			.VAlignCell(VAlign_Fill)
			.InitialSortMode(Column.GetInitialSortMode())
			.FillWidth(Column.GetInitialWidth())
			.HeaderContent()
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.Padding(FMargin(0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SEntityEventsTableView::GetColumnHeaderText, Column.GetId())
				]
			];

		int32 ColumnIndex = 0;
		const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
		const int32 NumColumns = HeaderRow->GetColumns().Num();
		for (; ColumnIndex < NumColumns; ColumnIndex++)
		{
			const SHeaderRow::FColumn& CurrentColumn = HeaderRow->GetColumns()[ColumnIndex];
			const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
			if (NewColumnPosition < CurrentColumnPosition)
			{
				break;
			}
		}

		HeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
	}

	FText SEntityEventsTableView::GetColumnHeaderText(const FName ColumnId) const
	{
		const UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		return Column.GetShortName();
	}

	void SEntityEventsTableView::InsightsManager_OnSessionChanged()
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		
		Session = CurrentSession;
		Reset();
	}

	void SEntityEventsTableView::Reset()
	{
	}
}

#undef LOCTEXT_NAMESPACE