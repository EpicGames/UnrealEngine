// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Common.h"
#include "SFragmentTableRow.h"
#include "Framework/Commands/UICommandList.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "InsightsCore/Table/ViewModels/Table.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace MassInsightsUI
{
	struct FOnSelectedEntityEventParams
	{
		uint64 ProviderEventIndex;
	};
	DECLARE_DELEGATE_OneParam(FOnSelectedEntityEvent, const FOnSelectedEntityEventParams&);

	/**
	 * Displays a list of events for the given entities between a configured time period
	 */
	class SEntityEventsTableView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SEntityEventsTableView) {}
			SLATE_EVENT(FOnSelectedEntityEvent, OnEntityEventSelected)
		SLATE_END_ARGS()

		SEntityEventsTableView();
		
		void Construct(const FArguments& args);
		void SetEntities(TConstArrayView<uint64> Entities);
	private:
	
		class FEventNode : public UE::Insights::FBaseTreeNode
		{
			INSIGHTS_DECLARE_RTTI(FEventNode, UE::Insights::FBaseTreeNode)
		public:
			explicit FEventNode(uint64 InProviderEntityIndex);

			virtual const FText GetDisplayName() const override;

			uint64 GetProviderEventIndex() const;

		private:
			uint64 ProviderEntityIndex;
		};

		friend class SEntityEventsTableRow;
		
		void InsightsManager_OnSessionChanged();

		void TreeView_OnSelectionChanged(TSharedPtr<FEventNode> EventNode, ESelectInfo::Type Arg) const;
		TSharedRef<ITableRow> TreeView_OnGenerateRow(TSharedPtr<FEventNode> NodePtr, const TSharedRef<STableViewBase>& OwnerTable);
		void TreeView_OnGetChildren(TSharedPtr<FEventNode> InParent, TArray<TSharedPtr<FEventNode>>& OutChildren);

		void IncrementalUpdate();
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		void ShowColumn(const FName ColumnId);
		FText GetColumnHeaderText(const FName ColumnId) const;

		void Reset();

		struct FEntityEventEntry
		{
			// Handle to link back to the data stored in the provider
			uint64 ProviderEventIndex;
			
			double EventTime;
			uint64 EntityId;

			// To appease treeview
			TSharedPtr<FEventNode> Handle;
		};
		
		TSharedPtr<UE::Insights::FTable> Table;
		TSharedPtr<const TraceServices::IAnalysisSession> Session;
		TSharedPtr<STreeView<TSharedPtr<FEventNode>>> TreeView;
		TSharedPtr<SHeaderRow> HeaderRow;
		
		TArray<TSharedPtr<FEventNode>> Events;
		TArray<TSharedPtr<FEventNode>> FilteredSortedEvents;

		bool bRebuildTable = true;
		// Keeps track of the incremental progress of events to process
		uint64 NextEventIndex = 0;
		
		TArray<uint64> Entities;
		
		MassInsightsUI::FOnSelectedArchetype OnArchetypeSelected;
		FOnSelectedEntityEvent OnEntityEventSelected;
	};
}
