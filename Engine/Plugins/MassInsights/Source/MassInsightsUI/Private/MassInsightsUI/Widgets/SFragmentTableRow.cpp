// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFragmentTableRow.h"

#include "Widgets/Text/STextBlock.h"

void MassInsights::SFragmentTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	TablePtr = InArgs._TablePtr;
	FragmentInfoPtr = InArgs._FragmentInfoPtr;
	SetEnabled(true);
	Super::Construct(Super::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> MassInsights::SFragmentTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	return SNew( STextBlock )
		.Text(FText::FromString( FragmentInfoPtr->Name ));
}
