// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InsightsCore/Table/ViewModels/Table.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

namespace UE::Mass::Trace
{
enum class EFragmentType : uint8;
}

namespace MassInsights
{
	struct FMassFragmentRowNodeData
	{
		uint64 Id;
		FString Name;
		uint32 Size;
		UE::Mass::Trace::EFragmentType Type;
	};
	using MassFragmentInfoPtr = TSharedPtr<FMassFragmentRowNodeData>;
	class SFragmentTableRow : public SMultiColumnTableRow< MassFragmentInfoPtr >
	{
		using Super = SMultiColumnTableRow< MassFragmentInfoPtr >;
	public:
		SLATE_BEGIN_ARGS(SFragmentTableRow) {}
			SLATE_ARGUMENT(TSharedPtr<UE::Insights::FTable>, TablePtr)
			SLATE_ARGUMENT(MassFragmentInfoPtr, FragmentInfoPtr)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	private:
		TSharedPtr<UE::Insights::FTable> TablePtr;
		MassFragmentInfoPtr FragmentInfoPtr;
		
	};
}
