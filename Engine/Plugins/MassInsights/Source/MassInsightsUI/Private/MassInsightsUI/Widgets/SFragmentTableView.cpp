// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFragmentTableView.h"

#include "Common/ProviderLock.h"
#include "Insights/IInsightsManager.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassTraceTypes.h"

#define LOCTEXT_NAMESPACE "MassInsights::SFragmentTableView"

namespace MassInsights
{
	SFragmentTableView::SFragmentTableView()
		: Table(MakeShared<UE::Insights::FTable>())
	{
	}

	SFragmentTableView::~SFragmentTableView()
	{
	}

	void SFragmentTableView::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				SAssignNew(TreeView, STreeView<MassInsights::MassFragmentInfoPtr>)
				.SelectionMode(ESelectionMode::Single)
				.TreeItemsSource(&FragmentInfos)
				.OnGetChildren(this, &SFragmentTableView::TreeView_OnGetChildren)
				.OnGenerateRow(this, &SFragmentTableView::TreeView_OnGenerateRow)
				.HeaderRow
				(
					SAssignNew(TreeViewHeaderRow, SHeaderRow)
					.Visibility(EVisibility::Visible)
				)
			]
		];

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
		if (InsightsManager.IsValid())
		{
			InsightsManager->GetSessionChangedEvent().AddSP(this, &SFragmentTableView::InsightsManager_OnSessionChanged);
			InsightsManager->GetSessionAnalysisCompletedEvent().AddSP(this, &SFragmentTableView::InsightsManager_OnSessionAnalysisComplete);
		}
		
		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		Session = CurrentSession;

		TArray<TSharedRef<UE::Insights::FTableColumn>> Columns;

		{
			using namespace UE::Insights;
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FName(TEXT("NameColumn")));
			FTableColumn& Column = *ColumnRef;
			Column.SetShortName(LOCTEXT("XX", "Name"));
			Column.SetTitleName(LOCTEXT("XX", "Name"));
			Column.SetDescription(LOCTEXT("XX", "Name"));
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(206.0f);
			Column.SetMinWidth(42.0f);
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible);
			Column.SetDataType(ETableCellDataType::Text);
			TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
			Column.SetValueGetter(Getter);

			TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
			Column.SetValueFormatter(Formatter);

			TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
			Column.SetValueSorter(Sorter);
			
			Column.Show();
			
			Columns.Add(ColumnRef);
		}
		
		Table->SetColumns(Columns);

		for (const TSharedRef<UE::Insights::FTableColumn>& ColumnRef : Table->GetColumns())
		{
			if (ColumnRef->ShouldBeVisible())
			{
				ShowColumn(ColumnRef->GetId());
			}
		}
		
	}

	void SFragmentTableView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// Incremental update
		RebuildTree(false);
	}

	void SFragmentTableView::InsightsManager_OnSessionChanged()
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
		
		Session = CurrentSession;
		Reset();
	}

	void SFragmentTableView::InsightsManager_OnSessionAnalysisComplete()
	{
	}

	TSharedRef<ITableRow> SFragmentTableView::TreeView_OnGenerateRow(
		MassInsights::MassFragmentInfoPtr NodePtr,
	    const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<ITableRow> TableRow =
			SNew(MassInsights::SFragmentTableRow, OwnerTable)
			.FragmentInfoPtr(NodePtr);
	
		return TableRow;
	}

	void SFragmentTableView::TreeView_OnGetChildren(
		MassInsights::MassFragmentInfoPtr InParent,
		TArray<MassInsights::MassFragmentInfoPtr>& OutChildren)
	{
		return;
	}

	void SFragmentTableView::Reset()
	{
		RebuildTree(true);
	}

	void SFragmentTableView::RebuildTree(bool bResync)
	{
		using namespace UE::Mass::Trace;

		if (bResync)
		{
			FragmentInfos.Reset();
		}
		const uint32 PreviousFragmentCount = FragmentInfos.Num();
		
		if (Session.IsValid())
		{
			const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);

			TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);
			const uint32 FragmentCount = static_cast<uint32>(Provider.GetFragmentCount());

			if (FragmentCount != PreviousFragmentCount)
			{
				FragmentInfos.SetNum(FragmentCount);
				Provider.EnumerateFragments([this](const FFragmentInfo& FragmentInfo, int32 Index)
				{
					MassFragmentInfoPtr InfoPtr = MakeShared<MassFragmentInfoPtr::ElementType>();
					InfoPtr->Id = FragmentInfo.Id;
					InfoPtr->Name = FragmentInfo.Name;
					InfoPtr->Size = FragmentInfo.Size;
					InfoPtr->Type = FragmentInfo.Type;
					FragmentInfos[Index] = InfoPtr;
				}, PreviousFragmentCount);
				
				TreeView->RebuildList();
			}
		}
	}

	void SFragmentTableView::ShowColumn(const FName ColumnId)
	{
		UE::Insights::FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
		Column.Show();

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
			.ColumnId(Column.GetId())
			.DefaultLabel(Column.GetShortName())
			.HAlignHeader(Column.GetHorizontalAlignment())
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Fill)
			.VAlignCell(VAlign_Fill)
			.InitialSortMode(Column.GetInitialSortMode())
			.FillWidth(Column.GetInitialWidth())
			.HeaderContent()
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.Padding(FMargin(0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Header Text")))
				]
			];

		int32 ColumnIndex = 0;
		const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
		const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
		for (; ColumnIndex < NumColumns; ColumnIndex++)
		{
			const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
			const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
			if (NewColumnPosition < CurrentColumnPosition)
			{
				break;
			}
		}

		TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
	}
}

#undef LOCTEXT_NAMESPACE