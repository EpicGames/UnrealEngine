// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SFragmentTableRow.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace MassInsights
{

class SFragmentTableView : public SCompoundWidget
{
public:
	SFragmentTableView();
	virtual ~SFragmentTableView();
	
	SLATE_BEGIN_ARGS(SFragmentTableView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void InsightsManager_OnSessionChanged();
	void InsightsManager_OnSessionAnalysisComplete();
	TSharedRef<ITableRow> TreeView_OnGenerateRow(MassInsights::MassFragmentInfoPtr NodePtr, const TSharedRef<STableViewBase>& OwnerTable);
	void TreeView_OnGetChildren(MassInsights::MassFragmentInfoPtr InParent, TArray<MassInsights::MassFragmentInfoPtr>& OutChildren);
	
	void Reset();
	void RebuildTree(bool bResync);
	void ShowColumn(const FName ColumnId);
private:
	TSharedPtr<UE::Insights::FTable> Table;
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
	TSharedPtr<FUICommandList> CommandList;
	
	TSharedPtr<STreeView<MassInsights::MassFragmentInfoPtr>> TreeView;
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	TArray<MassInsights::MassFragmentInfoPtr> FragmentInfos;
	TArray<MassInsights::MassFragmentInfoPtr> FilteredTreeItems;
};

}
