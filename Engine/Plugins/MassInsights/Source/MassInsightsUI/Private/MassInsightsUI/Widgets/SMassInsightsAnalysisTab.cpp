// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassInsightsAnalysisTab.h"

#include "SArchetypeDetails.h"
#include "SEntityEventsTableView.h"
#include "SFragmentTableView.h"
#include "Common/ProviderLock.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/ViewModels/ITimingEvent.h"
#include "Styling/AppStyle.h"
#include "Trace/MassTraceTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SNullWidget.h"
#include "MassInsightsUI/Widgets/SEntityEventsTableView.h"

#define LOCTEXT_NAMESPACE "SMassInsightsAnalysisTab"

namespace MassInsights
{
	void SMassInsightsAnalysisTab::Construct(const FArguments& InArgs)
	{
		using namespace UE::Mass::Trace;

		TSharedPtr<SHorizontalBox> ViewModeSelectorsBox;
		TSharedPtr<SWidgetSwitcher> ViewModeSwitcher;
		

		TSharedRef<MassInsightsUI::SArchetypeDetails> ArchetypeDetailsRef =
						SNew(MassInsightsUI::SArchetypeDetails);
		ArchetypesDetails = ArchetypeDetailsRef;

		TSharedRef<SEntityEventAggregationTableView> EntityTimelineTableViewRef =
			SNew(SEntityEventAggregationTableView)
			.OnArchetypeSelected_Lambda([this](uint64 ArchetypeID)
			{
				TSharedPtr<MassInsightsUI::SArchetypeDetails> Pinned = ArchetypesDetails.Pin();
				if (Pinned) { Pinned->SetArchetype(ArchetypeID); }
			})
			.OnRowSelected_Lambda([this](const FEntityEventSummaryRowSelectedParams& RowData)
			{
				TSharedPtr<MassInsightsUI::SEntityEventsTableView> Pinned = TableView.Pin();
				if (Pinned)
				{
					Pinned->SetEntities(MakeConstArrayView(&RowData.EntityID, 1));
				}

				// Add a time marker when the row is selected - will highlight the first event found
				// for that entity
				if (TimingViewSession)
				{
					if (RowData.IsSelected)
					{
						TimingViewSession->SetTimeMarker(RowData.FirstEventTime);
					}
				}
			});
		EntityTimelineTableView = EntityTimelineTableViewRef;
		
		TSharedRef<MassInsightsUI::SEntityEventsTableView> JourneyTableViewRef =
			SNew(MassInsightsUI::SEntityEventsTableView)
			.OnEntityEventSelected_Lambda([this](const MassInsightsUI::FOnSelectedEntityEventParams& Params)
			{
				TSharedPtr<MassInsightsUI::SArchetypeDetails> Pinned = ArchetypesDetails.Pin();
				if (Pinned
					&& AnalysisSession)
				{
					TValueOrError<FEntityEventRecord,void> Event = MakeError();
					const IMassTraceProvider& Provider = ReadMassTraceProvider(*AnalysisSession);
					{
						TraceServices::FProviderReadScopeLock ProviderReadScopeLock(Provider);

						Event = Provider.GetEntityEvent(Params.ProviderEventIndex);
					}

					if (Event.HasValue())
					{
						Pinned->SetArchetype(Event.GetValue().ArchetypeID);
						
						if (TimingViewSession)
						{
							TimingViewSession->SetTimeMarker(Event.GetValue().Time);
						}
					}
					
					
				}
			});
		TableView = JourneyTableViewRef;
		
		ChildSlot
		[
			SNew(SVerticalBox)

			// view mode selection
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8,2)
			[
				SAssignNew(ViewModeSelectorsBox, SHorizontalBox)
			]

			// active view mode panel
			+SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(3)
			[
				SAssignNew(ViewModeSwitcher, SWidgetSwitcher)
				.WidgetIndex(this, &SMassInsightsAnalysisTab::GetSelectedViewModeIndex)
			]
		];

		
		// helper to create widgets for a view mode
		auto AddViewMode = [this, &ViewModeSelectorsBox, &ViewModeSwitcher]( EViewMode InViewMode, const FText& InText)
		{
			// add mode selection button
			ViewModeSelectorsBox->AddSlot()
			.AutoWidth()
			.Padding(4)
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
				.IsChecked( this, &SMassInsightsAnalysisTab::IsViewModeSelected, InViewMode )
				.OnCheckStateChanged( this, &SMassInsightsAnalysisTab::OnViewModeCheckStateChange, InViewMode )
				[
					SNew(STextBlock)
						.Text(InText)
				]
			];

			// return a new slot for the mode's panel
			return ViewModeSwitcher->AddSlot();
		};

		
		
		AddViewMode(EViewMode::EntityEvents, LOCTEXT("ViewModeTab_Entities", "Entity Events"))
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)

			+SSplitter::Slot()
			.Value(0.6)
			[
				SNew(SBorder)
				.Padding(4)
				[
					EntityTimelineTableViewRef
				]
			]

			+SSplitter::Slot()
			.Value(0.4)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+SSplitter::Slot()
				.Value(0.5)
				[
					SNew(SBorder)
					.Padding(4)
					[
						JourneyTableViewRef
					]
				]
				+SSplitter::Slot()
				.Value(0.5)
				[
					SNew(SBorder)
					.Padding(4)
					[
						ArchetypeDetailsRef
					]
				]
			]
		];
		
		AddViewMode(EViewMode::Fragments, LOCTEXT("ViewMode_Fragments", "Fragments"))
		[
			SAssignNew(FragmentTableView, SFragmentTableView)
		];
	}

	void SMassInsightsAnalysisTab::SetSession(UE::Insights::Timing::ITimingViewSession* InTimingViewSession,
		const TraceServices::IAnalysisSession* InAnalysisSession)
	{
		if (TimingViewSession == InTimingViewSession)
		{
			return;
		}

		if (TimingViewSession)
		{
			TimingViewSession->OnSelectionChanged().RemoveAll(this);
			TimingViewSession->OnSelectedEventChanged().RemoveAll(this);
		}
		
		TimingViewSession = InTimingViewSession;
		AnalysisSession = InAnalysisSession;
	}

	int32 SMassInsightsAnalysisTab::GetSelectedViewModeIndex() const
	{
		return (int32)ViewMode;
	}

	ECheckBoxState SMassInsightsAnalysisTab::IsViewModeSelected(EViewMode InMode) const
	{
		return (InMode == ViewMode) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SMassInsightsAnalysisTab::OnViewModeCheckStateChange(ECheckBoxState State, EViewMode Mode)
	{
		if (State == ECheckBoxState::Checked)
		{
			ViewMode = Mode;
		}
	}

	bool SMassInsightsAnalysisTab::IsSessionSet() const
	{
		return TimingViewSession && AnalysisSession;
	}
}

#undef LOCTEXT_NAMESPACE