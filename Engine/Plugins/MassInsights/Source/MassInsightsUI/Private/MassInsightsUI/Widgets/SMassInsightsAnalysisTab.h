// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SArchetypeDetails.h"
#include "SEntityEventAggregationTableView.h"
#include "SFragmentTableView.h"
#include "Widgets/SCompoundWidget.h"

namespace MassInsightsUI
{
	class SEntityEventsTableView;
}

class ITimingEvent;

namespace UE::Insights::Timing
{
	class ITimingViewSession;
	enum class ETimeChangedFlags;
}

namespace TraceServices
{
	class IAnalysisSession;
}

namespace MassInsights
{
	class SMassInsightsAnalysisTab : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMassInsightsAnalysisTab) {}
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs);

		void SetSession(
			UE::Insights::Timing::ITimingViewSession* InTimingViewSession,
			const TraceServices::IAnalysisSession* InAnalysisSession);
		bool IsSessionSet() const;

	private:
		enum class EViewMode : uint8
		{
			EntityEvents,
			Fragments,
		};
		
		int32 GetSelectedViewModeIndex() const;
		ECheckBoxState IsViewModeSelected(EViewMode ViewMode) const;
		void OnViewModeCheckStateChange(ECheckBoxState State, EViewMode Mode);
		
		UE::Insights::Timing::ITimingViewSession* TimingViewSession = nullptr;
		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;

		TSharedPtr<SFragmentTableView> FragmentTableView;
		TSharedPtr<SEntityEventAggregationTableView> EntityTimelineTableView;
		TWeakPtr<MassInsightsUI::SArchetypeDetails> ArchetypesDetails;
		TWeakPtr<MassInsightsUI::SEntityEventsTableView> TableView;
		
		EViewMode ViewMode = EViewMode::EntityEvents;
	};
}