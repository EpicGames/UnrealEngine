// Copyright Epic Games, Inc. All Rights Reserved.
#include "MassInsightsUIModule.h"

#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Insights/ITimingViewExtender.h"
#include "Insights/IUnrealInsightsModule.h"
#include "MassInsightsUI/MassInsightsStyle.h"
#include "MassInsightsUI/Widgets/SFragmentTableView.h"
#include "MassInsightsUI/Widgets/SMassInsightsAnalysisTab.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MassInsightsModule"

namespace MassInsightsUI
{
	FMassInsightsUIModule& FMassInsightsUIModule::Get()
	{
		return FModuleManager::LoadModuleChecked<FMassInsightsUIModule>("MassInsightsUI");
	}
	
	void FMassInsightsUIModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
		TimingProfilerLayoutExtension.AddRaw(this, &FMassInsightsUIModule::RegisterLayoutExtension);
	}

	void FMassInsightsUIModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}

	TSharedPtr<MassInsights::SMassInsightsAnalysisTab> FMassInsightsUIModule::GetAnalysisTab()
	{
		return AnalysisTab.Pin();
	}

	namespace Private
	{
		const FName FrameViewTab("SlateFrame2ViewTab");
	}

	void FMassInsightsUIModule::RegisterLayoutExtension(FInsightsMajorTabExtender& Extender)
	{
		InsightsTabManager = Extender.GetTabManager();

		FMinorTabConfig& MinorTabConfig = Extender.AddMinorTabConfig();
		MinorTabConfig.TabId = Private::FrameViewTab;
		MinorTabConfig.TabLabel = LOCTEXT("MassInsightsTabTitle", "Mass Insights");
		MinorTabConfig.TabTooltip = LOCTEXT("MassInsightsTabTitleTooltip", "Tooltip here");
		MinorTabConfig.TabIcon = FSlateIcon(MassInsights::FMassInsightsStyle::Get().GetStyleSetName(), "MassInsights.Icon.Small");
		MinorTabConfig.WorkspaceGroup = Extender.GetWorkspaceGroup();
		MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				TSharedRef<MassInsights::SMassInsightsAnalysisTab> Content = SNew(MassInsights::SMassInsightsAnalysisTab);
				AnalysisTab = Content;
			
				const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.Padding(FMargin(2.0))
						[
							Content
						]					
					];

				return DockTab;
			});

		Extender.GetLayoutExtender().ExtendLayout(FTimingProfilerTabs::TimersID
			, ELayoutExtensionPosition::Before
			, FTabManager::FTab(Private::FrameViewTab, ETabState::ClosedTab));
	}
}


IMPLEMENT_MODULE(MassInsightsUI::FMassInsightsUIModule, MassInsightsUI);

#undef LOCTEXT_NAMESPACE
