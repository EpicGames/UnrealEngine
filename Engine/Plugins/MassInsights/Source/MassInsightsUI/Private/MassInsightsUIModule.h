// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"
#include "MassInsightsTimingTrack.h"

namespace MassInsights
{
	class SMassInsightsAnalysisTab;
}

struct FInsightsMajorTabExtender;
class FTabManager;

namespace MassInsightsUI
{
	

class FMassInsightsUIModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
	
	static FMassInsightsUIModule& Get();

	TSharedPtr<MassInsights::SMassInsightsAnalysisTab> GetAnalysisTab();

private:
	void RegisterLayoutExtension(FInsightsMajorTabExtender& Extender);
	FMassInsightsSharedState TimingViewExtender;
	TWeakPtr<FTabManager> InsightsTabManager;

	TWeakPtr<MassInsights::SMassInsightsAnalysisTab> AnalysisTab;
};

} //namespace MassInsightsUI